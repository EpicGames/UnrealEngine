// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module.h"
#include "VirtualScoutingLog.h"


#define LOCTEXT_NAMESPACE "FVirtualScoutingModule"


DEFINE_LOG_CATEGORY(LogVirtualScouting);


void FVirtualScoutingModule::StartupModule()
{
}


void FVirtualScoutingModule::ShutdownModule()
{
}


IMPLEMENT_MODULE(FVirtualScoutingModule, VirtualScouting)


#undef LOCTEXT_NAMESPACE
