// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualScoutingSettings.h"


namespace UE::VirtualScouting::Private
{
	static const FName PluginsCategoryName = TEXT("Plugins");
};


FName UVirtualScoutingSettings::GetCategoryName() const
{
	return UE::VirtualScouting::Private::PluginsCategoryName;
}

UVirtualScoutingSettings* UVirtualScoutingSettings::GetVirtualScoutingSettings()
{
	return GetMutableDefault<UVirtualScoutingSettings>();
}

FName UVirtualScoutingEditorSettings::GetCategoryName() const
{
	return UE::VirtualScouting::Private::PluginsCategoryName;
}

UVirtualScoutingEditorSettings* UVirtualScoutingEditorSettings::GetVirtualScoutingEditorSettings()
{
	return GetMutableDefault<UVirtualScoutingEditorSettings>();
}
