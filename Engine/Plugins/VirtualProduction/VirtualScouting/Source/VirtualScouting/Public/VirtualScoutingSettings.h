// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "VirtualScoutingSettings.generated.h"


/**
 * Per project settings for Virtual Scouting.
 */
UCLASS(Config=VirtualScoutingSettings, DefaultConfig, DisplayName="Virtual Scouting")
class VIRTUALSCOUTING_API UVirtualScoutingSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const override;
	//~ End UDeveloperSettings interface

	UFUNCTION(BlueprintPure, Category="Virtual Scouting", DisplayName="Virtual Scouting Settings")
	static UVirtualScoutingSettings* GetVirtualScoutingSettings();

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting", meta=(DisplayName="Show Measurements in Imperial Units"))
	bool bUseImperial = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting", meta=(DisplayName="Viewfinder Use AutoExposure"))
	bool bViewfinderUseExposure = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting", meta=(DisplayName="Swap to Grab on Spawn Actor"))
	bool bSwapToGrabToolOnSpawnNewActor = true;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting", meta=(DisplayName="Viewfinder ExposureCompensation", ClampMin=-15, ClampMax=15))
	float ViewfinderExposureCompensation = 1;
	
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting", meta=(DisplayName="Viewfinder Apertures"))
	TArray<float> ViewfinderApertureArray = {1.2f, 2.0f, 2.8f, 4.0f, 5.6f, 8.0f, 11.0f, 16.0f, 22.0f};

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting", meta=(DisplayName="Viewfinder Monitor Masks"))
	TArray<float> ViewfinderMaskArray = {1.33f, 1.66f, 1.78f, 2.0f, 2.35f, 2.39f };

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting", meta=(DisplayName="Sequence Tool Collection"))
	FName SequenceToolCollection;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Virtual Scouting", meta = (DisplayName = "Placement Tool Collection"))
	FName PlacementToolCollection;
};

/**
 * Per user settings for Virtual Scouting Editor.
 */
UCLASS(Config=EditorPerProjectUserSettings, DisplayName="Virtual Scouting Editor")
class VIRTUALSCOUTING_API UVirtualScoutingEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const override;
	//~ End UDeveloperSettings interface

	UFUNCTION(BlueprintPure, Category="Virtual Scouting Editor")
	static UVirtualScoutingEditorSettings* GetVirtualScoutingEditorSettings();

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting User", meta=(DisplayName= "Flight Speed", ToolTip="Avatar flying speed. Default is 4.0", ClampMin=1.0f, ClampMax=10.0f));
	float FlightSpeed = 4.0f;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting User", meta=(DisplayName= "Drag Speed", ToolTip="Speed of movement when you drag-move. Default is 0.7", ClampMin=0.1f, ClampMax=2.0f));
	float DragSpeed = 0.7f;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting User", meta=(DisplayName= "Show Tooltips", ToolTip="Show Tooltips when in VR. These appear when motioncontroller is brought near to HMD"));
	bool bEnableTooltips = true;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting User", meta=(DisplayName= "Use Smooth Rotation", ToolTip="True = Rotate smoothly. False = Flick Rotate. Default is Flick Rotate"));
	bool bUseSmoothRotation = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Virtual Scouting User", meta=(DisplayName= "Use Teleport Rotation", ToolTip="Use the forward axis roll from the motion controller to define and adjust teleport rotation"));
	bool bUseTeleportRotation = false;
};
