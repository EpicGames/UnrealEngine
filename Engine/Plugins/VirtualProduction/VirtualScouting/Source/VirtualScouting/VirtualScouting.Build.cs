// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VirtualScouting : ModuleRules
{
	public VirtualScouting(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CommonUI",
				"Core",
				"XRCreative",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"EnhancedInput",
				"HeadMountedDisplay",
				"InputCore",
				"InteractiveToolsFramework",
				"LevelSequence",
				"RenderCore", // TODO: Kill this? Only for FlushRenderingCommands
				"Slate",
				"SlateCore",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UMG",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"ConcertSyncClient",
				"ConcertSyncCore",
				"InputEditor",
				"LevelEditor",
				"MultiUserClient",
				"UnrealEd",
				"VREditor",
			});
		}
	}
}
