// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualScoutingEditorModule.h"
#include "Modules/ModuleManager.h"
#include "VRModeSettings.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


DEFINE_LOG_CATEGORY(LogVirtualScoutingEditor);


class FVirtualScoutingEditorModule : public IModuleInterface
{
private:
	virtual void StartupModule() override
	{
		// When the user has no previous selection upon enabling this plugin
		// for the first time, ensure we default to our preferred mode.
		UVRModeSettings* Settings = GetMutableDefault<UVRModeSettings>();
		if (Settings->ModeClass.IsNull())
		{
			Settings->ModeClass = FSoftObjectPath("/VirtualScouting/Core/Scouting_Default.Scouting_Default_C");
		}
	}

	virtual void ShutdownModule() override
	{
	}
};


IMPLEMENT_MODULE(FVirtualScoutingEditorModule, VirtualScoutingEditor);


PRAGMA_ENABLE_DEPRECATION_WARNINGS
