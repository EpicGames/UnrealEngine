// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VirtualScoutingEditor : ModuleRules
{
	public VirtualScoutingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		PrivateDependencyModuleNames.AddRange(new string[]
			{
				"OpenXRHMD",
				"OpenXRInput",
				"UnrealEd",
				"VREditor",
				"VirtualScoutingOpenXR",
				"EnhancedInput",
				"HeadMountedDisplay",
				"InputCore",
				"InputEditor",
				"InteractiveToolsFramework",
				"LevelEditor",
				"RenderCore",
				"Slate",
				"SlateCore",
				"UMG",
				"VirtualScouting",
				"XRCreative",
			}
		);
	}
}
