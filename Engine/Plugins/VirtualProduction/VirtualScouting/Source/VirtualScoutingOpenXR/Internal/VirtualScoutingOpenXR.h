// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "IOpenXRExtensionPlugin.h"
#include "Async/Future.h"
#include <openxr/openxr_platform_defines.h>

template <typename OptionalType> struct TOptional;


class VIRTUALSCOUTINGOPENXR_API FVirtualScoutingOpenXRExtension
	: public IOpenXRExtensionPlugin
	, public TSharedFromThis<FVirtualScoutingOpenXRExtension>
{
public:
	FVirtualScoutingOpenXRExtension();
	virtual ~FVirtualScoutingOpenXRExtension();

	//~ Begin IOpenXRExtensionPlugin interface
	virtual FString GetDisplayName() override
	{
		return FString(TEXT("VirtualScouting"));
	}

	virtual bool GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual void OnEvent(XrSession InSession, const XrEventDataBaseHeader* InHeader) override;
	virtual void PostCreateInstance(XrInstance InInstance) override;
	virtual void PostCreateSession(XrSession InSession) override;
	virtual void PostSyncActions(XrSession InSession) override;
	//~ End IOpenXRExtensionPlugin interface

	TFuture<FName>& GetHmdDeviceTypeFuture() { return DeviceTypeFuture; }

private:
	void OnVREditingModeEnter();
	void OnVREditingModeExit();

	TOptional<FName> TryGetHmdDeviceType();
	void TryFulfillDeviceTypePromise();

private:
	FDelegateHandle InitCompleteDelegate;

	XrDebugUtilsMessengerEXT Messenger = XR_NULL_HANDLE;

	XrInstance Instance = XR_NULL_HANDLE;
	XrSession Session = XR_NULL_HANDLE;
	XrActionSet ActionSet = XR_NULL_HANDLE;

	bool bIsVrEditingModeActive = false;

	TPromise<FName> DeviceTypePromise;
	TFuture<FName> DeviceTypeFuture;

private:
	static XrBool32 XRAPI_CALL XrDebugUtilsMessengerCallback_Trampoline(
		XrDebugUtilsMessageSeverityFlagsEXT InMessageSeverity,
		XrDebugUtilsMessageTypeFlagsEXT InMessageTypes,
		const XrDebugUtilsMessengerCallbackDataEXT* InCallbackData,
		void* InUserData);
};

#endif // #if WITH_EDITOR
