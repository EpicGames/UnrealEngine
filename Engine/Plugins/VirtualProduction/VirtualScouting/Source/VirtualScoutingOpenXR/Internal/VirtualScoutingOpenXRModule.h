// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"


class FVirtualScoutingOpenXRExtension;


class VIRTUALSCOUTINGOPENXR_API FVirtualScoutingOpenXRModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static FVirtualScoutingOpenXRModule& Get()
	{
		static const FName ModuleName = "VirtualScoutingOpenXR";
		return FModuleManager::LoadModuleChecked<FVirtualScoutingOpenXRModule>(ModuleName);
	};

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	const TSharedPtr<FVirtualScoutingOpenXRExtension>& GetOpenXRExt() { return OpenXRExt; }

private:
	TSharedPtr<FVirtualScoutingOpenXRExtension> OpenXRExt;
};
