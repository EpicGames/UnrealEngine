// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualScoutingOpenXR.h"
#include "Misc/CoreDelegates.h"
#include "VirtualScoutingOpenXRModule.h"

#include "OpenXRCore.h"

#if WITH_EDITOR

#include "IVREditorModule.h"


#define LOCTEXT_NAMESPACE "VirtualScouting"


DECLARE_LOG_CATEGORY_EXTERN(LogVirtualScoutingOpenXR, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogVirtualScoutingOpenXRDebug, VeryVerbose, All);
DEFINE_LOG_CATEGORY(LogVirtualScoutingOpenXR);
DEFINE_LOG_CATEGORY(LogVirtualScoutingOpenXRDebug);


static TAutoConsoleVariable<int32> CVarOpenXRDebugLogging(
	TEXT("VirtualScouting.OpenXRDebugLogging"),
	0,
	TEXT("If true, register an Unreal log sink via XR_EXT_debug_utils.\n"),
	ECVF_Default);



void FVirtualScoutingOpenXRModule::StartupModule()
{
	OpenXRExt = MakeShared<FVirtualScoutingOpenXRExtension>();
}

void FVirtualScoutingOpenXRModule::ShutdownModule()
{
	OpenXRExt.Reset();
}


IMPLEMENT_MODULE(FVirtualScoutingOpenXRModule, VirtualScoutingOpenXR);


FVirtualScoutingOpenXRExtension::FVirtualScoutingOpenXRExtension()
{
	RegisterOpenXRExtensionModularFeature();

	InitCompleteDelegate = FCoreDelegates::OnFEngineLoopInitComplete.AddLambda(
		[this]()
		{
			IVREditorModule& VrEditor = IVREditorModule::Get();
			VrEditor.OnVREditingModeEnter().AddRaw(this, &FVirtualScoutingOpenXRExtension::OnVREditingModeEnter);
			VrEditor.OnVREditingModeExit().AddRaw(this, &FVirtualScoutingOpenXRExtension::OnVREditingModeExit);

			// Must happen last; this implicitly deallocates this lambda's captures.
			FCoreDelegates::OnFEngineLoopInitComplete.Remove(InitCompleteDelegate);
		}
	);
}


FVirtualScoutingOpenXRExtension::~FVirtualScoutingOpenXRExtension()
{
	if (!DeviceTypeFuture.IsReady())
	{
		DeviceTypePromise.EmplaceValue(NAME_None);
	}

#if 0 // FIXME?: Too late to use Instance here, and don't see any suitable extension interface methods.
	// Might also be OK not to explicitly clean this up.
	if (Messenger != XR_NULL_HANDLE)
	{
		PFN_xrDestroyDebugUtilsMessengerEXT PfnXrDestroyDebugUtilsMessengerEXT;
		if (XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrDestroyDebugUtilsMessengerEXT",
			reinterpret_cast<PFN_xrVoidFunction*>(&PfnXrDestroyDebugUtilsMessengerEXT))))
		{
			XR_ENSURE(PfnXrDestroyDebugUtilsMessengerEXT(Messenger));
			Messenger = XR_NULL_HANDLE;
		}
	}
#endif

	UnregisterOpenXRExtensionModularFeature();

	if (IVREditorModule::IsAvailable())
	{
		IVREditorModule::Get().OnVREditingModeEnter().RemoveAll(this);
		IVREditorModule::Get().OnVREditingModeExit().RemoveAll(this);
	}
}


bool FVirtualScoutingOpenXRExtension::GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	if (CVarOpenXRDebugLogging.GetValueOnAnyThread() != 0)
	{
		OutExtensions.Add(XR_EXT_DEBUG_UTILS_EXTENSION_NAME);
	}

	return true;
}


void FVirtualScoutingOpenXRExtension::OnEvent(XrSession InSession, const XrEventDataBaseHeader* InHeader)
{
	switch (InHeader->type)
	{
		case XR_TYPE_EVENT_DATA_INTERACTION_PROFILE_CHANGED:
			// TODO: Correctly handle interaction profile changing mid-session
			ensure(InSession == Session);
			TryFulfillDeviceTypePromise();
			break;
	}
}


void FVirtualScoutingOpenXRExtension::PostCreateInstance(XrInstance InInstance)
{
	Instance = InInstance;

	if (CVarOpenXRDebugLogging.GetValueOnAnyThread() != 0)
	{
		PFN_xrCreateDebugUtilsMessengerEXT PfnXrCreateDebugUtilsMessengerEXT;
		if (XR_ENSURE(xrGetInstanceProcAddr(Instance, "xrCreateDebugUtilsMessengerEXT",
			reinterpret_cast<PFN_xrVoidFunction*>(&PfnXrCreateDebugUtilsMessengerEXT))))
		{
			XrDebugUtilsMessengerCreateInfoEXT DebugMessengerCreateInfo;
			DebugMessengerCreateInfo.type = XR_TYPE_DEBUG_UTILS_MESSENGER_CREATE_INFO_EXT;
			DebugMessengerCreateInfo.next = nullptr;
			DebugMessengerCreateInfo.messageSeverities =
				XR_DEBUG_UTILS_MESSAGE_SEVERITY_ERROR_BIT_EXT | XR_DEBUG_UTILS_MESSAGE_SEVERITY_WARNING_BIT_EXT |
				XR_DEBUG_UTILS_MESSAGE_SEVERITY_INFO_BIT_EXT | XR_DEBUG_UTILS_MESSAGE_SEVERITY_VERBOSE_BIT_EXT;
			DebugMessengerCreateInfo.messageTypes =
				XR_DEBUG_UTILS_MESSAGE_TYPE_GENERAL_BIT_EXT | XR_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT |
				XR_DEBUG_UTILS_MESSAGE_TYPE_PERFORMANCE_BIT_EXT | XR_DEBUG_UTILS_MESSAGE_TYPE_CONFORMANCE_BIT_EXT;
			DebugMessengerCreateInfo.userCallback = &FVirtualScoutingOpenXRExtension::XrDebugUtilsMessengerCallback_Trampoline;
			DebugMessengerCreateInfo.userData = this;

			if (XR_ENSURE(PfnXrCreateDebugUtilsMessengerEXT(Instance, &DebugMessengerCreateInfo, &Messenger)))
			{
				UE_LOGF(LogVirtualScoutingOpenXRDebug, Log, "XR_EXT_debug_utils messenger ACTIVE");
				return;
			}
		}
	}

	UE_LOGF(LogVirtualScoutingOpenXRDebug, Log, "XR_EXT_debug_utils messenger DISABLED");
}


void FVirtualScoutingOpenXRExtension::PostCreateSession(XrSession InSession)
{
	Session = InSession;

	DeviceTypePromise = {};
	DeviceTypeFuture = DeviceTypePromise.GetFuture();
}


void FVirtualScoutingOpenXRExtension::PostSyncActions(XrSession InSession)
{
	TryFulfillDeviceTypePromise();
}


void FVirtualScoutingOpenXRExtension::TryFulfillDeviceTypePromise()
{
	if (DeviceTypeFuture.IsValid() && !DeviceTypeFuture.IsReady())
	{
		TOptional<FName> MaybeType = TryGetHmdDeviceType();
		if (MaybeType)
		{
			DeviceTypePromise.EmplaceValue(MaybeType.GetValue());
			return;
		}
	}
}


TOptional<FName> FVirtualScoutingOpenXRExtension::TryGetHmdDeviceType()
{
	if (Instance == XR_NULL_HANDLE || Session == XR_NULL_HANDLE)
	{
		return FName();
	}

	XrPath LeftHand = XR_NULL_PATH, RightHand = XR_NULL_PATH;
	check(XR_SUCCEEDED(xrStringToPath(Instance, "/user/hand/left", &LeftHand)));
	check(XR_SUCCEEDED(xrStringToPath(Instance, "/user/hand/right", &RightHand)));

	XrInteractionProfileState LeftHandState{ XR_TYPE_INTERACTION_PROFILE_STATE },
	                          RightHandState{ XR_TYPE_INTERACTION_PROFILE_STATE };

	XR_ENSURE(xrGetCurrentInteractionProfile(Session, LeftHand, &LeftHandState));
	XR_ENSURE(xrGetCurrentInteractionProfile(Session, RightHand, &RightHandState));

	// TODO: Correctly handle mixed interaction profiles?
	XrPath InteractionProfile = LeftHandState.interactionProfile != XR_NULL_PATH
		? LeftHandState.interactionProfile
		: RightHandState.interactionProfile;

	if (InteractionProfile == XR_NULL_PATH)
	{
		// Returning an unset optional indicates we don't know _yet_, and the caller can poll again.
		return TOptional<FName>();
	}

	uint32 PathLen;
	char PathBuf[XR_MAX_PATH_LENGTH] = { 0 };
	if (!XR_ENSURE(xrPathToString(Instance, InteractionProfile, XR_MAX_PATH_LENGTH, &PathLen, PathBuf)))
	{
		return FName();
	}

	const FAnsiStringView PathView(PathBuf, PathLen - 1);
	if (PathView.StartsWith(ANSITEXTVIEW("/interaction_profiles/oculus/")))
	{
		return FName("OculusHMD");
	}
	else if (PathView.StartsWith(ANSITEXTVIEW("/interaction_profiles/htc/"))
		|| PathView.StartsWith(ANSITEXTVIEW("/interaction_profiles/valve/")))
	{
		return FName("SteamVR");
	}

	UE_LOGF(LogVirtualScoutingOpenXR, Error, "Unsupported interaction profile: '%s'", PathBuf);
	return FName();
}


void FVirtualScoutingOpenXRExtension::OnVREditingModeEnter()
{
	bIsVrEditingModeActive = true;
}


void FVirtualScoutingOpenXRExtension::OnVREditingModeExit()
{
	bIsVrEditingModeActive = false;
}


//static
XrBool32 XRAPI_CALL FVirtualScoutingOpenXRExtension::XrDebugUtilsMessengerCallback_Trampoline(
	XrDebugUtilsMessageSeverityFlagsEXT InMessageSeverity,
	XrDebugUtilsMessageTypeFlagsEXT InMessageTypes,
	const XrDebugUtilsMessengerCallbackDataEXT* InCallbackData,
	void* InUserData)
{
	// TODO?: InUserData contains the FVirtualScoutingOpenXRExtension*, could forward to an instance method.
	// However, doing the work directly in here seems fine for now.

	ELogVerbosity::Type Verbosity;
	switch (InMessageSeverity)
	{
		case XR_DEBUG_UTILS_MESSAGE_SEVERITY_VERBOSE_BIT_EXT: Verbosity = ELogVerbosity::Verbose; break;
		case XR_DEBUG_UTILS_MESSAGE_SEVERITY_INFO_BIT_EXT:    Verbosity = ELogVerbosity::Display; break;
		case XR_DEBUG_UTILS_MESSAGE_SEVERITY_WARNING_BIT_EXT: Verbosity = ELogVerbosity::Warning; break;
		case XR_DEBUG_UTILS_MESSAGE_SEVERITY_ERROR_BIT_EXT:   Verbosity = ELogVerbosity::Error;   break;
		default:
			ensureMsgf(false, TEXT("Unhandled XrDebugUtilsMessageSeverityFlagsEXT: %llX"), InMessageSeverity);
			Verbosity = ELogVerbosity::Error;
			break;
	}

	// Results in "____" if no bits are set, or "GVPC" if all bits are set, or some combination.
	FString Types = TEXT("####");
	Types[0] = (InMessageTypes & XR_DEBUG_UTILS_MESSAGE_TYPE_GENERAL_BIT_EXT)     ? TEXT('G') : TEXT('_');
	Types[1] = (InMessageTypes & XR_DEBUG_UTILS_MESSAGE_TYPE_VALIDATION_BIT_EXT)  ? TEXT('V') : TEXT('_');
	Types[2] = (InMessageTypes & XR_DEBUG_UTILS_MESSAGE_TYPE_PERFORMANCE_BIT_EXT) ? TEXT('P') : TEXT('_');
	Types[3] = (InMessageTypes & XR_DEBUG_UTILS_MESSAGE_TYPE_CONFORMANCE_BIT_EXT) ? TEXT('C') : TEXT('_');

	FMsg::Logf(__FILE__, __LINE__, LogVirtualScoutingOpenXRDebug.GetCategoryName(), Verbosity,
		TEXT("[%s]: %S(): %S"), *Types, InCallbackData->functionName, InCallbackData->message);

	// "A value of XR_TRUE indicates that the application wants to abort this call. [...]
	// Applications should always return XR_FALSE so that they see the same behavior with
	// and without validation layers enabled."
	return XR_FALSE;
}


#undef LOCTEXT_NAMESPACE

#else // not WITH_EDITOR

IMPLEMENT_MODULE(FDefaultModuleImpl, VirtualScoutingOpenXR);

#endif // #if WITH_EDITOR .. #else
