// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VirtualScoutingOpenXR : ModuleRules
{
	public VirtualScoutingOpenXR(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"OpenXRHMD",
			}
		);

		// This entire module is an Editor module in spirit, but in practice must be Runtime,
		// because IOpenXRExtensionPlugin requires the PostConfigInit LoadingPhase.
		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.Add("VREditor");
		}

		PrivateDependencyModuleNames.AddRange(new string[]
			{
			}
		);
	}
}
