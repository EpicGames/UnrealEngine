{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Virtual Scouting",
	"Description": "Virtual Scouting lets filmmakers scout a digital environment in virtual reality.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "VirtualScoutingOpenXR",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "VirtualScouting",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "VirtualScoutingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "XRCreativeFramework",
			"Enabled":  true
		},
		{
			"Name": "OpenXR",
			"Enabled": true
		},
		{
			"Name": "EnhancedInput",
			"Enabled": true
		},
		{
			"Name": "CommonUI",
			"Enabled": true
		},
		{
			"Name": "BlueprintFileUtils",
			"Enabled": true
		},
		{
			"Name": "JsonBlueprintUtilities",
			"Enabled": true
		},
		{
			"Name": "MultiUserClient",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "GeometryScripting",
			"Enabled": true
		},
		{
			"Name": "GizmoFramework",
			"Enabled": true
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true
		},
		{
			"Name": "ConsoleVariables",
			"Enabled": true
		},
		{
			"Name": "LevelSnapshots",
			"Enabled": true
		},
		{
			"Name": "CineCameraSceneCapture",
			"Enabled": true
		},
		{
			"Name": "VirtualProductionUtilities",
			"Enabled": true
		}
	]
}
